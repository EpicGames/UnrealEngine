// Copyright Epic Games, Inc. All Rights Reserved.

#include "PosixShim.h"

#include <time.h>
#include <atomic>

namespace
{
	/**
	 * Static storage function for our local clock correction seconds. We use static storage to ensure
	 * we don't allocate anything until used.
	 */
	static std::atomic<time_t>& GetCurrentLocalClockCorrectionSeconds()
	{
		static std::atomic<time_t> LocalClockCorrectionSeconds(0);
		return LocalClockCorrectionSeconds;
	}
}

void SetPosixShimUTCServerTime(const FDateTime UTCServerTime)
{
	const time_t LocalUtcTimeSeconds = time(nullptr);
	const time_t LocalClockCorrectionAmountSeconds = UTCServerTime.ToUnixTimestamp() - LocalUtcTimeSeconds;

	GetCurrentLocalClockCorrectionSeconds().store(LocalClockCorrectionAmountSeconds);
}

time_t UE_epoch_time_offset()
{
	return GetCurrentLocalClockCorrectionSeconds().load();
}
