// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// any configuration / setup calls would go here

/**
 * Sets the current UTC server time.  This value should be set from a trusted-to-be-roughly-correct source, and
 * should be updated from time to time.
 *
 * @param UTCServerTime The new server time
 */
POSIXSHIM_API void SetPosixShimUTCServerTime(const FDateTime UTCServerTime);
