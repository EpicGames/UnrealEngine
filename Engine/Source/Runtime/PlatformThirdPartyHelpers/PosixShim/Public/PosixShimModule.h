// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

/**
 * PosixShimModule interface
 */
class IPosixShimModule : public IModuleInterface
{
};
